// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "SXcodeGPUDebuggerPluginEditorExtension.h"

#include "XcodeGPUDebuggerPluginCommands.h"
#include "XcodeGPUDebuggerPluginModule.h"
#include "XcodeGPUDebuggerPluginStyle.h"

#include "Editor/EditorEngine.h"
#include "Editor/UnrealEd/Public/SEditorViewportToolBarMenu.h"
#include "Editor/UnrealEd/Public/SViewportToolBarComboMenu.h"
#include "Editor/UnrealEd/Public/Kismet2/DebuggerCommands.h"
#include "RHI.h"
#include "Styling/AppStyle.h"
#include "Framework/Application/SlateApplication.h"
#include "Framework/MultiBox/MultiBoxBuilder.h"
#include "Framework/MultiBox/MultiBoxDefs.h"
#include "Interfaces/IMainFrameModule.h"
#include "LevelEditor.h"
#include "ToolMenus.h"
#include "ViewportToolbar/UnrealEdViewportToolbar.h"
#include "Widgets/Input/SButton.h"
#include "Widgets/Images/SImage.h"

extern UNREALED_API UEditorEngine* GEditor;

FXcodeGPUDebuggerPluginEditorExtension::FXcodeGPUDebuggerPluginEditorExtension(FXcodeGPUDebuggerPluginModule* ThePlugin)
	: LoadedDelegateHandle()
	, IsEditorInitialized(false)
{
	// Defer Level Editor UI extensions until Level Editor has been loaded:
	if (FModuleManager::Get().IsModuleLoaded("LevelEditor"))
	{
		Initialize(ThePlugin);
	}
	else
	{
		FModuleManager::Get().OnModulesChanged().AddLambda([this, ThePlugin](FName name, EModuleChangeReason reason)
		{
			if ((name == "LevelEditor") && (reason == EModuleChangeReason::ModuleLoaded))
			{
				Initialize(ThePlugin);
			}
		});
	}
}

FXcodeGPUDebuggerPluginEditorExtension::~FXcodeGPUDebuggerPluginEditorExtension()
{
	FXcodeGPUDebuggerPluginStyle::Shutdown();
	FXcodeGPUDebuggerPluginCommands::Unregister();
	UToolMenus::UnregisterOwner(this);
}

void FXcodeGPUDebuggerPluginEditorExtension::Initialize(FXcodeGPUDebuggerPluginModule* ThePlugin)
{
	if (GUsingNullRHI)
	{
		return;
	}

	// The LoadModule request below will crash if running as an editor commandlet!
	check(!IsRunningCommandlet());

	FXcodeGPUDebuggerPluginStyle::Initialize();
	FXcodeGPUDebuggerPluginCommands::Register();

	FToolMenuOwnerScoped ScopedOwner(this);

	UToolMenu* Menu = UToolMenus::Get()->ExtendMenu("LevelEditor.ViewportToolbar");
	
	FToolMenuSection& RightSection = Menu->FindOrAddSection("Right");
	FToolMenuEntry& Entry = RightSection.AddMenuEntry("XCodeCaptureFrame", FXcodeGPUDebuggerPluginCommands::Get().CaptureFrame);
	Entry.ToolBarData.LabelOverride = FText::GetEmpty();
	Entry.InsertPosition.Position = EToolMenuInsertType::First;

	IsEditorInitialized = false;
	FSlateRenderer* SlateRenderer = FSlateApplication::Get().GetRenderer();
	LoadedDelegateHandle = SlateRenderer->OnSlateWindowRendered().AddRaw(this, &FXcodeGPUDebuggerPluginEditorExtension::OnEditorLoaded);
}

void FXcodeGPUDebuggerPluginEditorExtension::OnEditorLoaded(SWindow& SlateWindow)
{
	// Would be nice to use the preprocessor definition WITH_EDITOR instead, but the user may launch a standalone the game through the editor...
	if (GEditor == nullptr)
	{
		return;
	}

	if (IsInGameThread())
	{
		FSlateRenderer* SlateRenderer = FSlateApplication::Get().GetRenderer();
		SlateRenderer->OnSlateWindowRendered().Remove(LoadedDelegateHandle);
	}

	if (IsEditorInitialized)
	{
		return;
	}
	IsEditorInitialized = true;

	if(FPlayWorldCommands::GlobalPlayWorldActions.IsValid())
	{
		//Register the editor hotkeys
		FPlayWorldCommands::GlobalPlayWorldActions->MapAction(FXcodeGPUDebuggerPluginCommands::Get().CaptureFrame,
			FExecuteAction::CreateLambda([]()
			{
				FXcodeGPUDebuggerPluginModule& PluginModule = FModuleManager::GetModuleChecked<FXcodeGPUDebuggerPluginModule>("XcodeGPUDebuggerPlugin");
				PluginModule.CaptureFrame(nullptr, IRenderCaptureProvider::ECaptureFlags_Launch, FString());
			}),
			FCanExecuteAction());
 	}
}

#endif //WITH_EDITOR
