// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Editor/UnrealEd/Public/SViewportToolBar.h"

class FXcodeGPUDebuggerPluginModule;
class FExtensionBase;
class FExtensibilityManager;
class FExtender;
class FToolBarBuilder;

class FXcodeGPUDebuggerPluginEditorExtension
{
public:
	FXcodeGPUDebuggerPluginEditorExtension(FXcodeGPUDebuggerPluginModule* ThePlugin);
	~FXcodeGPUDebuggerPluginEditorExtension();

private:
	void Initialize(FXcodeGPUDebuggerPluginModule* ThePlugin);
	void OnEditorLoaded(SWindow& SlateWindow);

	FDelegateHandle LoadedDelegateHandle;
	bool IsEditorInitialized;
};

#endif // WITH_EDITOR
