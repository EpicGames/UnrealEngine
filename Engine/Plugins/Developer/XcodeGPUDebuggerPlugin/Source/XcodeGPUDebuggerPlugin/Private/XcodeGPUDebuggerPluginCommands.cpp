// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "XcodeGPUDebuggerPluginCommands.h"

#define LOCTEXT_NAMESPACE "XcodeGPUDebuggerPlugin"

void FXcodeGPUDebuggerPluginCommands::RegisterCommands()
{
	UI_COMMAND(CaptureFrame, "Capture Frame", "Captures the next frame and launches Xcode (Shift+E)", EUserInterfaceActionType::Button, FInputChord(EKeys::E, EModifierKey::Shift));
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_EDITOR
