// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "XcodeGPUDebuggerPluginStyle.h"
#include "Framework/Commands/Commands.h"

class FXcodeGPUDebuggerPluginCommands : public TCommands<FXcodeGPUDebuggerPluginCommands>
{
public:
	FXcodeGPUDebuggerPluginCommands()
		: TCommands<FXcodeGPUDebuggerPluginCommands>(TEXT("XcodeGPUDebuggerPlugin"), NSLOCTEXT("Contexts", "XcodeGPUDebuggerPlugin", "XcodeGPUDebugger Plugin"), NAME_None, FXcodeGPUDebuggerPluginStyle::Get()->GetStyleSetName())
	{
	}

	virtual void RegisterCommands() override;

	TSharedPtr<class FUICommandInfo> CaptureFrame;
};

#endif // WITH_EDITOR
