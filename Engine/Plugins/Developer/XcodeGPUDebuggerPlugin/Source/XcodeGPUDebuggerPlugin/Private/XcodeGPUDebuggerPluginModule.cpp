// Copyright Epic Games, Inc. All Rights Reserved.

#include "XcodeGPUDebuggerPluginModule.h"

#include "Engine/Engine.h"
#include "Engine/GameViewportClient.h"
#include "Async/Async.h"
#include "HAL/FileManager.h"
#include "Internationalization/Internationalization.h"
#include "Misc/ConfigCacheIni.h"
#include "RendererInterface.h"
#include "RenderingThread.h"
#include "UnrealClient.h"
#include "GeneralProjectSettings.h"
#include "IMetalDynamicRHI.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/EditorEngine.h"
#include "ISettingsModule.h"
#include "ISettingsSection.h"
#include "LevelEditor.h"
#include "SXcodeGPUDebuggerPluginEditorExtension.h"
extern UNREALED_API UEditorEngine* GEditor;
#endif // WITH_EDITOR

#include "Apple/AppleStringUtils.h"
#include "MetalInclude.h"

DEFINE_LOG_CATEGORY(XcodeGPUDebuggerPlugin);

#define LOCTEXT_NAMESPACE "XcodeGPUDebuggerPlugin"

struct FXcodeGPUDebuggerAsyncGraphTask : public FAsyncGraphTaskBase
{
	ENamedThreads::Type TargetThread;
	TFunction<void()> TheTask;

	FXcodeGPUDebuggerAsyncGraphTask(ENamedThreads::Type Thread, TFunction<void()>&& Task) : TargetThread(Thread), TheTask(MoveTemp(Task)) { }
	void DoTask(ENamedThreads::Type CurrentThread, const FGraphEventRef& MyCompletionGraphEvent) { TheTask(); }
	ENamedThreads::Type GetDesiredThread() { return(TargetThread); }
};

class FXcodeGPUDebuggerFrameCapturer
{
public:
	static FString MakeXcodeGPUDebuggerCaptureFilePath(FString const& InFileName)
	{
		if (InFileName.IsEmpty())
		{
			return InFileName;
		}

		const bool bAbsoluteFileName = !FPaths::IsRelative(InFileName);
		FString FileName = bAbsoluteFileName ? InFileName : FPaths::ConvertRelativePathToFull(FPaths::ProjectSavedDir() / FString("XcodeGPUTraceCaptures") / InFileName);
		FileName = FPaths::SetExtension(FileName, "gputrace");
		FPaths::MakePlatformFilename(FileName);
		return FileName;
	}

	static void BeginFrameCapture(const FString& CaptureFileName)
	{
		UE4_GEmitDrawEvents_BeforeCapture = GetEmitDrawEvents();
		SetEmitDrawEvents(true);

		MTL::Device* MetalDevice = GetIMetalDynamicRHI()->RHIGetDevice();
		MTL::CaptureManager* CaptureManager = MTL::CaptureManager::sharedCaptureManager();

		MTL::CaptureDescriptor* CaptureDescriptor = MTL::CaptureDescriptor::alloc()->init();
        
		CaptureDescriptor->setCaptureObject((__bridge id<MTLDevice>)MetalDevice);

#if PLATFORM_IOS || PLATFORM_TVOS
		// On iOS/tvOS, assume that Xcode is attached.
		CaptureDescriptor->setDestination(MTL::CaptureDestinationDeveloperTools);
#else
		FString SaveURL = FXcodeGPUDebuggerFrameCapturer::MakeXcodeGPUDebuggerCaptureFilePath(CaptureFileName);
		FTCHARToUTF8 TCUrlStr(*SaveURL);
		NS::String* NSUrlStr = (NS::String*)SaveURL.GetCFString();
		NS::URL* Url = NS::URL::alloc()->initFileURLWithPath(NSUrlStr);

		CaptureDescriptor->setDestination(MTL::CaptureDestinationGPUTraceDocument);
		CaptureDescriptor->setOutputURL(Url);
#endif

		NS::Error* Error = nullptr;
		if (!CaptureManager->startCapture(CaptureDescriptor,&Error))
		{
			FString ErrorString((CFStringRef)Error->localizedDescription());
			UE_LOGF(XcodeGPUDebuggerPlugin, Warning, "Failed to start capture (%ls)", *ErrorString);
		}
	}

	static void EndFrameCapture(FRHICommandListImmediate& RHICmdList)
	{
		RHICmdList.SubmitAndBlockUntilGPUIdle();

		MTL::CaptureManager* CaptureManager = MTL::CaptureManager::sharedCaptureManager();
		CaptureManager->stopCapture();

		SetEmitDrawEvents(UE4_GEmitDrawEvents_BeforeCapture);
	}

	static void SaveAndLaunch(FXcodeGPUDebuggerPluginModule* Plugin, int32 Flags, const FString& InDestPath)
	{
#if !(PLATFORM_IOS || PLATFORM_TVOS)
		FString DestPath = MakeXcodeGPUDebuggerCaptureFilePath(InDestPath);

		if (Flags & IRenderCaptureProvider::ECaptureFlags_Launch)
		{
			TGraphTask<FXcodeGPUDebuggerAsyncGraphTask>::CreateTask().ConstructAndDispatchWhenReady(ENamedThreads::GameThread, [DestPath]()
			{
				FString XcodePath = FPlatformMisc::GetXcodePath();
				XcodePath.RemoveFromEnd(TEXT("/Contents/Developer"));

				NSURL* GPUTracePath = [NSURL fileURLWithPath:[NSString stringWithFString:DestPath]];

				[[NSWorkspace sharedWorkspace]
					openURLs:[NSArray arrayWithObject: GPUTracePath]
					withApplicationAtURL:[NSURL URLWithString:XcodePath.GetNSString()]
					configuration: [NSWorkspaceOpenConfiguration configuration]
					completionHandler:nil
				];
			});
		}
#endif
	}

private:
	static bool UE4_GEmitDrawEvents_BeforeCapture;
};

bool FXcodeGPUDebuggerFrameCapturer::UE4_GEmitDrawEvents_BeforeCapture = false;

class FXcodeGPUDebuggerDummyInputDevice : public IInputDevice
{
public:
	FXcodeGPUDebuggerDummyInputDevice(FXcodeGPUDebuggerPluginModule* InPlugin) : ThePlugin(InPlugin) { }
	virtual ~FXcodeGPUDebuggerDummyInputDevice() { }

	/** Tick the interface (used for controlling full engine frame captures). */
	virtual void Tick(float DeltaTime) override
	{
		check(ThePlugin);
		ThePlugin->Tick(DeltaTime);
	}

	/** The remaining interfaces are irrelevant for this dummy input device. */
	virtual void SendControllerEvents() override { }
	virtual void SetMessageHandler(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override { }
	virtual bool Exec(UWorld* InWorld, const TCHAR* Cmd, FOutputDevice& Ar) override { return(false); }
	virtual void SetChannelValue(int32 ControllerId, FForceFeedbackChannelType ChannelType, float Value) override { }
	virtual void SetChannelValues(int32 ControllerId, const FForceFeedbackValues &values) override { }

private:
	FXcodeGPUDebuggerPluginModule* ThePlugin;
};

TSharedPtr<class IInputDevice> FXcodeGPUDebuggerPluginModule::CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler)
{
	UE_LOGF(XcodeGPUDebuggerPlugin, Log, "Creating dummy input device (for intercepting engine ticks)");
	FXcodeGPUDebuggerDummyInputDevice* InputDev = new FXcodeGPUDebuggerDummyInputDevice(this);
	return MakeShareable((IInputDevice*)InputDev);
}

void FXcodeGPUDebuggerPluginModule::StartupModule()
{
#if !UE_BUILD_SHIPPING // Disable in shipping builds
#if WITH_EDITOR
	EditorExtensions = nullptr;
#endif // WITH_EDITOR

	// Enable Metal capture layer.
	setenv("METAL_CAPTURE_ENABLED", "1", 1);

	InjectDebugExecKeybind();

	// Do not load the plugin if UE4 is headless.
	if (GUsingNullRHI)
	{
		UE_LOGF(XcodeGPUDebuggerPlugin, Display, "Xcode GPU Debugger plugin will not be loaded (headless UE4 instance).");
		return;
	}

	IModularFeatures::Get().RegisterModularFeature(IRenderCaptureProvider::GetModularFeatureName(), (IRenderCaptureProvider*)this);
	IModularFeatures::Get().RegisterModularFeature(IInputDeviceModule::GetModularFeatureName(), (IInputDeviceModule*)this);

	bCaptureInProgress = false;

	// Setup XcodeGPUDebugger settings
	FString XcodeGPUDebuggerCapturePath = FPaths::ProjectSavedDir() / TEXT("XcodeGPUTraceCaptures");
	if (!IFileManager::Get().DirectoryExists(*XcodeGPUDebuggerCapturePath))
	{
		IFileManager::Get().MakeDirectory(*XcodeGPUDebuggerCapturePath, true);
	}

	// Register console commands.
	static FAutoConsoleCommand CCmdXcodeGPUDebuggerCaptureFrame = FAutoConsoleCommand(
		TEXT("Xcode.CaptureFrame"),
		TEXT("Captures the rendering commands of the next frame and launches Xcode"),
		FConsoleCommandDelegate::CreateRaw(this, &FXcodeGPUDebuggerPluginModule::CaptureFrame)
	);

#if WITH_EDITOR
	EditorExtensions = new FXcodeGPUDebuggerPluginEditorExtension(this);
#endif // WITH_EDITOR

	UE_LOGF(XcodeGPUDebuggerPlugin, Log, "Xcode GPU Debugger plugin is ready!");
#endif // !UE_BUILD_SHIPPING
}

void FXcodeGPUDebuggerPluginModule::BeginFrameCapture(const FString& InCaptureFileName)
{
	UE_LOGF(XcodeGPUDebuggerPlugin, Log, "Capturing the frame and running Xcode (to file '%ls.gputrace')", *InCaptureFileName);

	ENQUEUE_RENDER_COMMAND(StartXcodeGPUDebuggerCapture)(
		[InCaptureFileName](FRHICommandListImmediate& RHICmdList)
		{
			FXcodeGPUDebuggerFrameCapturer::BeginFrameCapture(InCaptureFileName);
		}
    );
}

void FXcodeGPUDebuggerPluginModule::InjectDebugExecKeybind()
{
	GConfig->AddUniqueToSection(TEXT("/Script/Engine.PlayerInput"), TEXT("DebugExecBindings"), TEXT("(Key=E,Command=\"Xcode.CaptureFrame\", Shift=true)"), GInputIni);
}

void FXcodeGPUDebuggerPluginModule::EndFrameCapture(void* HWnd, uint32 Flags, const FString& DestFileName)
{
	ENQUEUE_RENDER_COMMAND(EndXcodeGPUDebuggerCapture)(
		[Plugin = this, Flags, DestFileName](FRHICommandListImmediate& RHICmdList)
		{
			FXcodeGPUDebuggerFrameCapturer::EndFrameCapture(RHICmdList);
			FXcodeGPUDebuggerFrameCapturer::SaveAndLaunch(Plugin, Flags, DestFileName);
		});

	bCaptureInProgress = false;
}

void FXcodeGPUDebuggerPluginModule::CaptureFrame(FViewport* InViewport, uint32 InFlags, FString const& InDestFileName)
{
	check(IsInGameThread());

	// Don't do anything if we're currently already waiting for a capture to end.
	if (bCaptureInProgress)
	{
		return;
	}

    DoFrameCaptureCurrentViewport(InViewport, InFlags, InDestFileName);
}

void FXcodeGPUDebuggerPluginModule::CaptureFrame()
{
	CaptureFrame(nullptr, ECaptureFlags_Launch, FString());
}

void FXcodeGPUDebuggerPluginModule::DoFrameCaptureCurrentViewport(FViewport* InViewport, uint32 InFlags, const FString& InDestFileName)
{
	CaptureFileName = InDestFileName;

	// Use the current date/time as a fallback.
	if (CaptureFileName.IsEmpty())
	{
		CaptureFileName = FDateTime::Now().ToString();
	}

	BeginFrameCapture(CaptureFileName);

	// infer the intended viewport to intercept/capture:
	FViewport* Viewport = InViewport;

	check(GEngine);
	if (!Viewport && GEngine->GameViewport)
	{
		check(GEngine->GameViewport->Viewport);
		if (GEngine->GameViewport->Viewport->HasFocus())
		{
			Viewport = GEngine->GameViewport->Viewport;
		}
	}

#if WITH_EDITOR
	if (!Viewport && GEditor)
	{
		// WARNING: capturing from a "PIE-Eject" Editor viewport will not work as
		// expected; in such case, capture via the console command
		// (this has something to do with the 'active' editor viewport when the UI
		// button is clicked versus the one which the console is attached to)
		Viewport = GEditor->GetActiveViewport();
	}
#endif // WITH_EDITOR

	check(Viewport);
	Viewport->Draw(true);

	EndFrameCapture(Viewport->GetWindow(), InFlags, CaptureFileName);
}

void FXcodeGPUDebuggerPluginModule::Tick(float DeltaTime)
{
	if (bCaptureInProgress)
	{
		EndFrameCapture(nullptr, ECaptureFlags_Launch, FString());
	}
}

void FXcodeGPUDebuggerPluginModule::StartXcode(FString LaunchPath)
{
	if (LaunchPath.IsEmpty())
	{
		return;
	}

	FString XcodePath = FPlatformMisc::GetXcodePath();
	XcodePath.RemoveFromEnd(TEXT("/Contents/Developer"));

	NSURL* GPUTracePath = [NSURL fileURLWithPath:[NSString stringWithFString:LaunchPath]];

	[[NSWorkspace sharedWorkspace]
		openURLs:[NSArray arrayWithObject: GPUTracePath]
		withApplicationAtURL:[NSURL URLWithString:XcodePath.GetNSString()]
		configuration: [NSWorkspaceOpenConfiguration configuration]
		completionHandler:nil
	];
}

void FXcodeGPUDebuggerPluginModule::ShutdownModule()
{
    if (GUsingNullRHI)
    {
        return;
    }

	IModularFeatures::Get().UnregisterModularFeature(IRenderCaptureProvider::GetModularFeatureName(), (IRenderCaptureProvider*)this);

#if WITH_EDITOR
	delete EditorExtensions;
#endif // WITH_EDITOR
}

void FXcodeGPUDebuggerPluginModule::BeginCapture(FRHICommandListImmediate* InRHICommandList, uint32 InFlags, FString const& InDestFileName)
{
	CaptureFlags = InFlags;
	CaptureFileName = InDestFileName;

	// Use the current date/time as a fallback.
	if (CaptureFileName.IsEmpty())
	{
		CaptureFileName = FDateTime::Now().ToString();
	}
	
	MTL::Device* MetalDevice = GetIMetalDynamicRHI()->RHIGetDevice();

	InRHICommandList->SubmitAndBlockUntilGPUIdle();
	InRHICommandList->EnqueueLambda([FileName = CaptureFileName, MetalDevice](FRHICommandListImmediate& RHICommandList)
	{
		MTL::CaptureManager* CaptureManager = MTL::CaptureManager::sharedCaptureManager();

		MTL::CaptureDescriptor* CaptureDescriptor = MTL::CaptureDescriptor::alloc()->init();
		CaptureDescriptor->setCaptureObject((__bridge id<MTLDevice>)MetalDevice);

#if PLATFORM_IOS || PLATFORM_TVOS
		// On iOS/tvOS, assume that Xcode is attached.
		CaptureDescriptor->setDestination(MTL::CaptureDestinationDeveloperTools);
#else
		FString SaveURL = FXcodeGPUDebuggerFrameCapturer::MakeXcodeGPUDebuggerCaptureFilePath(FileName);
		NS::String* NSUrlStr = (NS::String*)SaveURL.GetCFString();
		NS::URL* Url = NS::URL::fileURLWithPath(NSUrlStr);

		CaptureDescriptor->setDestination(MTL::CaptureDestinationGPUTraceDocument);
		CaptureDescriptor->setOutputURL(Url);
#endif

		NS::Error* Error;
		if (!CaptureManager->startCapture(CaptureDescriptor, &Error))
		{
			FString ErrorString((CFStringRef)Error->localizedDescription());
			UE_LOGF(XcodeGPUDebuggerPlugin, Warning, "Failed to start capture (%ls)", *ErrorString);
		}
	});
}

void FXcodeGPUDebuggerPluginModule::EndCapture(FRHICommandListImmediate* InRHICommandList)
{
	InRHICommandList->SubmitAndBlockUntilGPUIdle();
    InRHICommandList->EnqueueLambda([this](FRHICommandListImmediate& RHICommandList)
	{
		MTL::CaptureManager* CaptureManager = MTL::CaptureManager::sharedCaptureManager();
		CaptureManager->stopCapture();

		FXcodeGPUDebuggerFrameCapturer::SaveAndLaunch(this, CaptureFlags, CaptureFileName);
	});

	CaptureFlags = 0;
	CaptureFileName.Reset();
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FXcodeGPUDebuggerPluginModule, XcodeGPUDebuggerPlugin)
