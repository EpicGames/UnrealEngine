// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IXcodeGPUDebuggerPlugin.h"

DECLARE_LOG_CATEGORY_EXTERN(XcodeGPUDebuggerPlugin, Log, All);

class FXcodeGPUDebuggerPluginEditorExtension;

class FXcodeGPUDebuggerPluginModule : public IXcodeGPUDebuggerPlugin
{
	friend class FXcodeGPUDebuggerDummyInputDevice;
	friend class FXcodeGPUDebuggerFrameCapturer;

public:
	// Begin IRenderCaptureProvider interface.
	virtual void CaptureFrame(FViewport* InViewport, uint32 InFlags, FString const& InDestFileName) override;
	virtual void BeginCapture(FRHICommandListImmediate* InRHICommandList, uint32 InFlags, FString const& InDestFileName) override;
	virtual void EndCapture(FRHICommandListImmediate* InRHICommandList) override;
	// End IRenderCaptureProvider interface.

protected:
	// Begin IModuleInterface interface.
	virtual void StartupModule() override;
	virtual void ShutdownModule() override;
	// End IModuleInterface interface.

	// Begin IInputDeviceModule interface.
	virtual TSharedPtr<class IInputDevice> CreateInputDevice(const TSharedRef<FGenericApplicationMessageHandler>& InMessageHandler) override;
	// End IInputDeviceModule interface.

private:
	void Tick(float DeltaTime);

	/** Injects a debug key bind into the local player so that the hot key works the same in game */
	void InjectDebugExecKeybind();

	/** Helper function for CVar command binding. */
	void CaptureFrame();

	void DoFrameCaptureCurrentViewport(FViewport* InViewport, uint32 InFlags, FString const& InDestFileName);

	void BeginFrameCapture(const FString& InCaptureFileName);
	void EndFrameCapture(void* HWnd, uint32 Flags, FString const& DestFileName);

	void StartXcode(FString CapturePath);

private:
    uint32 CaptureFlags;
    uint32 bCaptureInProgress:1;

	FString CaptureFileName;

#if WITH_EDITOR
	FXcodeGPUDebuggerPluginEditorExtension* EditorExtensions;
#endif // WITH_EDITOR
};
