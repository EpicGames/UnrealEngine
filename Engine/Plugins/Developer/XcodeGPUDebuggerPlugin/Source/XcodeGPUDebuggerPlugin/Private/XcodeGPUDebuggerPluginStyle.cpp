// Copyright Epic Games, Inc. All Rights Reserved.

#if WITH_EDITOR

#include "XcodeGPUDebuggerPluginStyle.h"
#include "Styling/SlateStyle.h"
#include "Styling/SlateStyleRegistry.h"
#include "HAL/FileManager.h"
#include "Interfaces/IPluginManager.h"

FString FXcodeGPUDebuggerPluginStyle::InContent(const FString& RelativePath, const ANSICHAR* Extension)
{
	auto myself = IPluginManager::Get().FindPlugin(TEXT("XcodeGPUDebuggerPlugin"));
	check(myself.IsValid());
	static FString ContentDir = myself->GetBaseDir() / TEXT("Resources");
	return (ContentDir / RelativePath) + Extension;
}

TSharedPtr<FSlateStyleSet> FXcodeGPUDebuggerPluginStyle::StyleSet = nullptr;
TSharedPtr<class ISlateStyle> FXcodeGPUDebuggerPluginStyle::Get() { return StyleSet; }

void FXcodeGPUDebuggerPluginStyle::Initialize()
{
	// Only register once
	if (StyleSet.IsValid())
	{
		return;
	}

	StyleSet = MakeShareable(new FSlateStyleSet("XcodeGPUDebuggerPluginStyle"));

	FString ProjectResourceDir = FPaths::ProjectPluginsDir() / TEXT("XcodeGPUDebuggerPlugin/Resources");
	FString EngineResourceDir = FPaths::EnginePluginsDir() / TEXT("XcodeGPUDebuggerPlugin/Resources");

	if (IFileManager::Get().DirectoryExists(*ProjectResourceDir)) //Is the plugin in the project? In that case, use those resources
	{
		StyleSet->SetContentRoot(ProjectResourceDir);
		StyleSet->SetCoreContentRoot(ProjectResourceDir);
	}
	else //Otherwise, use the global ones
	{
		StyleSet->SetContentRoot(EngineResourceDir);
		StyleSet->SetCoreContentRoot(EngineResourceDir);
	}

	StyleSet->Set("XcodeGPUDebuggerPlugin.Icon", new FSlateImageBrush(FXcodeGPUDebuggerPluginStyle::InContent("Icon128", ".png"), FVector2D(128.0f, 128.0f)));
	StyleSet->Set("XcodeGPUDebuggerPlugin.CaptureFrame", new FSlateImageBrush(FXcodeGPUDebuggerPluginStyle::InContent("ViewportIcon16", ".png"), FVector2D(16.0f, 16.0f)));

	FSlateStyleRegistry::RegisterSlateStyle(*StyleSet.Get());
};

void FXcodeGPUDebuggerPluginStyle::Shutdown()
{
	if (StyleSet.IsValid())
	{
		FSlateStyleRegistry::UnRegisterSlateStyle(*StyleSet.Get());
		ensure(StyleSet.IsUnique());
		StyleSet.Reset();
	}
}

#endif //WITH_EDITOR
