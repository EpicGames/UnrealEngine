// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IInputDeviceModule.h"
#include "IRenderCaptureProvider.h"

class IXcodeGPUDebuggerPlugin : public IInputDeviceModule, public IRenderCaptureProvider
{
public:
	static inline IXcodeGPUDebuggerPlugin& Get()
	{
		return FModuleManager::LoadModuleChecked<IXcodeGPUDebuggerPlugin>("XcodeGPUDebugger");
	}

	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("XcodeGPUDebugger");
	}
};
