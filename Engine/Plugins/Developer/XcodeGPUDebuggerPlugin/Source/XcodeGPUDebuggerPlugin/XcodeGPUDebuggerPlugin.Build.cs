// Copyright Epic Games, Inc. All Rights Reserved.

using System.IO;

namespace UnrealBuildTool.Rules
{
	public class XcodeGPUDebuggerPlugin : ModuleRules
	{
		public XcodeGPUDebuggerPlugin(ReadOnlyTargetRules Target) : base(Target)
		{
			bRequiresPlatformSDK = true;
			
			PrivateDependencyModuleNames.AddRange(new string[] { });
			PublicDependencyModuleNames.AddRange(new string[]
			{
				  "Core"
				, "CoreUObject"
				, "Engine"
				, "InputCore"
				, "DesktopPlatform"
				, "Projects"
				, "RenderCore"
				, "InputDevice"
				, "RHI"
				, "DeveloperSettings"
				, "MetalRHI"
			});

			AddEngineThirdPartyPrivateStaticDependencies(Target, "MetalCPP");
			PublicWeakFrameworks.Add("Metal");

			if (Target.bBuildEditor == true)
			{
				DynamicallyLoadedModuleNames.AddRange(new string[] { "LevelEditor" });
				PublicDependencyModuleNames.AddRange(new string[]
				{
					  "Slate"
					, "SlateCore"
					, "UnrealEd"
					, "MainFrame"
					, "GameProjectGeneration"
					, "ToolMenus"
				});
			}
		}
	}
}
