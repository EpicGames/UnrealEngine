{
	"FileVersion" : 1,
	"FriendlyName" : "Xcode GPU Debugger Plugin",
	"Version" : 1,
	"VersionName" : "1.0",
	"CreatedBy" : "Apple, Inc.",
	"CreatedByURL" : "https://apple.com/",
	"DocsURL" : "",
	"Description" : "Xcode GPU debugger integration.",
	"Category" : "Rendering",
	"EnabledByDefault" : false,
	"Modules" :
	[
		{
			"Name": "XcodeGPUDebuggerPlugin",
			"Type": "DeveloperTool",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList":
            [
				"Mac"
			]
		}
	]
}
